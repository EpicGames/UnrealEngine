{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Image Plate",
	"Description" : "Actor and component types that provide a camera-aligned image plate",
	"Category" : "Rendering",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "ImagePlate",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "ImagePlateEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}