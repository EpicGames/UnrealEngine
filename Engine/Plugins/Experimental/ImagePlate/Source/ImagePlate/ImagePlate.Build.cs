// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ImagePlate : ModuleRules
	{
		public ImagePlate(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ImageCore",
					"ImageWrapper",
					"MediaAssets",
					"RenderCore",
					"RHI",
					"TimeManagement",
                    "SlateCore",
                }
			);

			PrivateIncludePaths.AddRange(
				new string[] {
				}
			);
		}
	}
}
