// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImagePlate.h"
#include "ImagePlateComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImagePlate)

AImagePlate::AImagePlate(const FObjectInitializer& Init)
	: Super(Init)
{
	ImagePlate = Init.CreateDefaultSubobject<UImagePlateComponent>(this, "ImagePlateComponent");
	RootComponent = ImagePlate;
}
