// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImagePlateComponent.h"

#include "UObject/ConstructorHelpers.h"
#include "Engine/Texture.h"
#include "PrimitiveViewRelevance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "PrimitiveSceneProxy.h"
#include "Engine/CollisionProfile.h"
#include "RayTracingInstance.h"
#include "PrimitiveDrawingUtils.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "Engine/Engine.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "DynamicMeshBuilder.h"
#include "Components/SceneCaptureComponent2D.h"
#include "ImagePlateFrustumComponent.h"
#include "ImagePlateComponent.h"
#include "RenderUtils.h"
#include "SceneInterface.h"
#include "StaticMeshResources.h"
#include "Kismet/GameplayStatics.h"
#include "RHIResourceUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImagePlateComponent)

namespace
{

	static TAutoConsoleVariable<int32> CVarRayTracingImagePlate(
		TEXT("r.RayTracing.Geometry.ImagePlate"),
		1,
		TEXT("Include image plate meshes in ray tracing effects (default = 1 (image plates enabled in ray tracing))"));


	class FImagePlateIndexBuffer : public FIndexBuffer
	{
	public:
		TArray<uint16> Indices;

		virtual void InitRHI(FRHICommandListBase& RHICmdList) override
		{
			IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FImagePlateIndexBuffer"), EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource, MakeConstArrayView(Indices));
		}
	};

	/** Represents a sprite to the scene manager. */
	class FImagePlateSceneProxy final : public FPrimitiveSceneProxy
	{
	public:
		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		/** Initialization constructor. */
		FImagePlateSceneProxy(UImagePlateComponent* InComponent)
			: FPrimitiveSceneProxy(InComponent)
			, VertexFactory(GetScene().GetFeatureLevel(), "FImagePlateSceneProxy")
		{
			Material = InComponent->GetPlate().DynamicMaterial ? InComponent->GetPlate().DynamicMaterial : InComponent->GetPlate().Material;
			if (Material)
			{
				MaterialRelevance |= Material->GetRelevance_Concurrent(GetScene().GetShaderPlatform());
			}

#if RHI_RAYTRACING
			bHasRayTracingRepresentation = (Material != nullptr) && IsRayTracingEnabled();
#endif // RHI_RAYTRACING
		}

		~FImagePlateSceneProxy()
		{
			VertexBuffers.PositionVertexBuffer.ReleaseResource();
			VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
			VertexBuffers.ColorVertexBuffer.ReleaseResource();
			IndexBuffer.ReleaseResource();
			VertexFactory.ReleaseResource();
#if RHI_RAYTRACING
			RayTracingGeometry.ReleaseResource();
#endif // RHI_RAYTRACING
		}

		virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override
		{
			BuildMesh(RHICmdList);
			IndexBuffer.InitResource(RHICmdList);

#if RHI_RAYTRACING
			if (bHasRayTracingRepresentation)
			{
				FRayTracingGeometryInitializer Initializer;
				Initializer.DebugName = GetOwnerName();
				Initializer.IndexBuffer = IndexBuffer.IndexBufferRHI;
				Initializer.TotalPrimitiveCount = IndexBuffer.Indices.Num() / 3;
				Initializer.GeometryType = RTGT_Triangles;
				Initializer.bFastBuild = true;
				Initializer.bAllowUpdate = false;

				FRayTracingGeometrySegment Segment;
				Segment.VertexBuffer = VertexBuffers.PositionVertexBuffer.VertexBufferRHI;
				Segment.MaxVertices = VertexBuffers.PositionVertexBuffer.GetNumVertices();
				Segment.NumPrimitives = Initializer.TotalPrimitiveCount;

				Initializer.Segments.Add(Segment);

				RayTracingGeometry.SetInitializer(Initializer);
				RayTracingGeometry.InitResource(RHICmdList);
			}
#endif // RHI_RAYTRACING
		}

		void BuildMesh(FRHICommandListBase& RHICmdList)
		{
			TArray<FDynamicMeshVertex> Vertices;
			Vertices.Empty(4);
			Vertices.AddUninitialized(4);

			// Set up the sprite vertex positions and texture coordinates.
			Vertices[0].Position  = FVector3f(0, -1.f,  1.f);
			Vertices[1].Position  = FVector3f(0, -1.f, -1.f);
			Vertices[2].Position  = FVector3f(0,  1.f,  1.f);
			Vertices[3].Position  = FVector3f(0,  1.f, -1.f);

			Vertices[0].TextureCoordinate[0] = FVector2f(0,0);
			Vertices[1].TextureCoordinate[0] = FVector2f(0,1);
			Vertices[2].TextureCoordinate[0] = FVector2f(1,0);
			Vertices[3].TextureCoordinate[0] = FVector2f(1,1);

			VertexBuffers.InitFromDynamicVertex(RHICmdList, &VertexFactory, Vertices);

			IndexBuffer.Indices.Empty(6);
			IndexBuffer.Indices.AddUninitialized(6);

			IndexBuffer.Indices[0] = 0;
			IndexBuffer.Indices[1] = 1;
			IndexBuffer.Indices[2] = 2;
			IndexBuffer.Indices[3] = 1;
			IndexBuffer.Indices[4] = 2;
			IndexBuffer.Indices[5] = 3;
		}

		virtual void DrawStaticElements(FStaticPrimitiveDrawInterface* PDI) override
		{
			QUICK_SCOPE_CYCLE_COUNTER( STAT_ImagePlateSceneProxy_DrawStaticElements );

			if (Material)
			{
				FMeshBatch Mesh;
				Mesh.VertexFactory           = &VertexFactory;
				Mesh.MaterialRenderProxy     = Material->GetRenderProxy();
				Mesh.ReverseCulling          = IsLocalToWorldDeterminantNegative();
				Mesh.CastShadow              = false;
				Mesh.DepthPriorityGroup      = SDPG_World;
				Mesh.Type                    = PT_TriangleList;
				Mesh.bDisableBackfaceCulling = true;
				Mesh.LODIndex                = 0;

				FMeshBatchElement& BatchElement = Mesh.Elements[0];
				BatchElement.IndexBuffer = &IndexBuffer;
				BatchElement.FirstIndex             = 0;
				BatchElement.MinVertexIndex         = 0;
				BatchElement.MaxVertexIndex         = 3;
				BatchElement.NumPrimitives          = 2;

				PDI->DrawMesh(Mesh, 1.f);
			}
		}

		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			QUICK_SCOPE_CYCLE_COUNTER( STAT_ImagePlateSceneProxy_GetDynamicMeshElements );

			if (!Material)
			{
				return;
			}

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (!(VisibilityMap & (1 << ViewIndex)))
				{
					continue;
				}

				const FSceneView* View = Views[ViewIndex];

				// Draw the mesh.
				FMeshBatch& Mesh = Collector.AllocateMesh();
				Mesh.VertexFactory = &VertexFactory;
				Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
				Mesh.CastShadow = false;
				Mesh.bDisableBackfaceCulling = false;
				Mesh.Type = PT_TriangleList;
				Mesh.DepthPriorityGroup = (ESceneDepthPriorityGroup)GetDepthPriorityGroup(View);
				Mesh.bCanApplyViewModeOverrides = true;
				Mesh.MaterialRenderProxy = Material->GetRenderProxy();

				FMeshBatchElement& BatchElement = Mesh.Elements[0];
				BatchElement.IndexBuffer = &IndexBuffer;
				BatchElement.FirstIndex			= 0;
				BatchElement.MinVertexIndex		= 0;
				BatchElement.MaxVertexIndex		= 3;
				BatchElement.NumPrimitives		= 2;

				Collector.AddMesh(ViewIndex, Mesh);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				RenderBounds(Collector.GetPDI(ViewIndex), View->Family->EngineShowFlags, GetBounds(), IsSelected());
#endif
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = IsShown(View);
			Result.bRenderCustomDepth = ShouldRenderCustomDepth();
			Result.bRenderInMainPass = ShouldRenderInMainPass();
			Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
			Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;

			Result.bShadowRelevance = IsShadowCast(View);
			if( IsRichView(*View->Family) ||
				View->Family->EngineShowFlags.Bounds ||
				View->Family->EngineShowFlags.Collision ||
				IsSelected() ||
				IsHovered())
			{
				Result.bDynamicRelevance = true;
			}
			else
			{
				Result.bStaticRelevance = true;
			}
			
			MaterialRelevance.SetPrimitiveViewRelevance(Result);

			Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;

			return Result;
		}
		virtual bool CanBeOccluded() const override { return !MaterialRelevance.bDisableDepthTest; }
		virtual uint32 GetMemoryFootprint() const override { return sizeof(*this) + GetAllocatedSize(); }
		uint32 GetAllocatedSize() const { return FPrimitiveSceneProxy::GetAllocatedSize(); }

#if RHI_RAYTRACING
		virtual bool IsRayTracingRelevant() const override { return true; }

		virtual bool HasRayTracingRepresentation() const override { return bHasRayTracingRepresentation; }

		virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override final
		{
			if (!CVarRayTracingImagePlate.GetValueOnRenderThread())
			{
				return;
			}

			TConstArrayView<const FSceneView*> Views = Collector.GetViews();
			const uint32 VisibilityMap = Collector.GetVisibilityMap();

			// RT geometry will be generated based on first active view and then reused for all other views
			// TODO: Expose a way for developers to control whether to reuse RT geometry or create one per-view
			const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
			checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

			const FSceneView* FirstActiveView = Views[FirstActiveViewIndex];

			FMaterialRenderProxy* MaterialProxy = Material->GetRenderProxy();

			if (RayTracingGeometry.IsValid())
			{
				check(RayTracingGeometry.GetInitializer().IndexBuffer.IsValid());

				FRayTracingInstance RayTracingInstance;
				RayTracingInstance.Geometry = &RayTracingGeometry;
				RayTracingInstance.InstanceTransforms.Add(GetLocalToWorld());

				uint32 SectionIdx = 0;
				FMeshBatch MeshBatch;

				MeshBatch.VertexFactory = &VertexFactory;
				MeshBatch.SegmentIndex = 0;
				MeshBatch.MaterialRenderProxy = MaterialProxy;
				MeshBatch.ReverseCulling = IsLocalToWorldDeterminantNegative();
				MeshBatch.Type = PT_TriangleList;
				MeshBatch.DepthPriorityGroup = SDPG_World;
				MeshBatch.bCanApplyViewModeOverrides = false;
				MeshBatch.CastRayTracedShadow = IsShadowCast(FirstActiveView);

				FMeshBatchElement& BatchElement = MeshBatch.Elements[0];
				BatchElement.IndexBuffer = &IndexBuffer;

				FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
				FPrimitiveUniformShaderParametersBuilder Builder;
				BuildUniformShaderParameters(Builder);
				DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), Builder);

				BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

				BatchElement.FirstIndex = 0;
				BatchElement.NumPrimitives = IndexBuffer.Indices.Num() / 3;
				BatchElement.MinVertexIndex = 0;
				BatchElement.MaxVertexIndex = VertexBuffers.PositionVertexBuffer.GetNumVertices() - 1;

				RayTracingInstance.Materials.Add(MeshBatch);

				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
				{
					if ((VisibilityMap & (1 << ViewIndex)) == 0)
					{
						continue;
					}

					Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);
				}
			}
		}
#endif // RHI_RAYTRACING

	private:
		UMaterialInterface* Material;
		FMaterialRelevance MaterialRelevance;
		FStaticMeshVertexBuffers VertexBuffers;
		FImagePlateIndexBuffer IndexBuffer;
		FLocalVertexFactory VertexFactory;
#if RHI_RAYTRACING
		FRayTracingGeometry RayTracingGeometry;
		bool bHasRayTracingRepresentation;
#endif // RHI_RAYTRACING
	};
}

FImagePlateParameters::FImagePlateParameters()
	: Material(nullptr)
	, TextureParameterName("InputTexture")
	, bFillScreen(true)
	, FillScreenAmount(100, 100)
	, FixedSize(100,100)
	, RenderTexture(nullptr)
	, DynamicMaterial(nullptr)
{
}

UImagePlateComponent::UImagePlateComponent(const FObjectInitializer& Init)
	: Super(Init)
{
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> DefaultImagePlateMaterial(TEXT("/ImagePlate/DefaultImagePlateMaterial"));
	Plate.Material = DefaultImagePlateMaterial.Object;

	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	bUseAsOccluder = false;
	bTickInEditor = true;
	PrimaryComponentTick.bCanEverTick = true;
	bReenetrantTranformChange = false;
}

void UImagePlateComponent::OnRegister()
{
	Super::OnRegister();
	UpdateMaterialParametersForMedia();
	UpdateTransformScale();

#if WITH_EDITORONLY_DATA
	if (AActor* ComponentOwner = GetOwner())
	{
		if (!EditorFrustum)
		{
			EditorFrustum = NewObject<UImagePlateFrustumComponent>(ComponentOwner, NAME_None, RF_Transactional | RF_TextExportTransient);
			EditorFrustum->SetupAttachment(this);
			/*EditorFrustum->bIsEditorOnly = true;*/
			EditorFrustum->CreationMethod = CreationMethod;
			EditorFrustum->RegisterComponentWithWorld(GetWorld());
		}
	}
#endif
}

void UImagePlateComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	UpdateTransformScale();
}

void UImagePlateComponent::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	Super::OnUpdateTransform(UpdateTransformFlags, Teleport);

	if (!bReenetrantTranformChange)
	{
		bReenetrantTranformChange = true;
		UpdateTransformScale();
		bReenetrantTranformChange = false;
	}
}

void UImagePlateComponent::SetImagePlate(FImagePlateParameters NewPlate)
{
	Plate = NewPlate;

	// Interpret a null material to mean the default material
	if (!Plate.Material)
	{
		Plate.Material = GetDefault<UImagePlateComponent>()->Plate.Material;
	}

	UpdateMaterialParametersForMedia();
}

void UImagePlateComponent::OnRenderTextureChanged()
{
	UpdateMaterialParametersForMedia();
}

void UImagePlateComponent::UpdateTransformScale()
{
	// Cache the view projection matrices of our target
	AActor* ViewTarget = FindViewTarget();
	if (ViewTarget && Plate.bFillScreen)
	{
		FMatrix ViewMatrix;
		FMatrix ProjectionMatrix;
		UGameplayStatics::CalculateViewProjectionMatricesFromViewTarget(ViewTarget, ViewMatrix, ProjectionMatrix, ViewProjectionMatrix);
		InvViewProjectionMatrix = ViewProjectionMatrix.Inverse();

		const FMatrix LocalToWorld = GetComponentTransform().ToMatrixNoScale();
		const FMatrix WorldToLocal = LocalToWorld.Inverse();
		const FMatrix ScreenToLocalSpace = ViewProjectionMatrix.Inverse() * WorldToLocal;


		// Just use the current view projection matrix
		FVector4 HGLocalPosition = (LocalToWorld * ViewProjectionMatrix).TransformPosition(FVector::ZeroVector);
		FVector ScreenSpaceLocalPosition = HGLocalPosition;
		if (HGLocalPosition.W != 0.0f)
		{
			ScreenSpaceLocalPosition /= HGLocalPosition.W;
		}

		FVector HorizontalScale		= UImagePlateComponent::TransfromFromProjection(ScreenToLocalSpace, FVector4(Plate.FillScreenAmount.X/100.f, 0.f, ScreenSpaceLocalPosition.Z, 1.0f));
		FVector VerticalScale		= UImagePlateComponent::TransfromFromProjection(ScreenToLocalSpace, FVector4(0.f, Plate.FillScreenAmount.Y/100.f, ScreenSpaceLocalPosition.Z, 1.0f));

		SetRelativeScale3D(FVector(GetRelativeScale3D().X, HorizontalScale.Size(), VerticalScale.Size()));
		SetRelativeLocation(FVector(GetRelativeLocation().X, 0.f, 0.f));
	}
	else
	{
		SetRelativeScale3D(FVector(GetRelativeScale3D().X, Plate.FixedSize.X*.5f, Plate.FixedSize.Y*.5f));
	}
}

void UImagePlateComponent::UpdateMaterialParametersForMedia()
{
	if (!Plate.TextureParameterName.IsNone() && Plate.Material && Plate.RenderTexture)
	{
		if (!Plate.DynamicMaterial)
		{
			Plate.DynamicMaterial = UMaterialInstanceDynamic::Create(Plate.Material, this);
			Plate.DynamicMaterial->SetFlags(RF_Transient);
		}

		Plate.DynamicMaterial->SetTextureParameterValue(Plate.TextureParameterName, Plate.RenderTexture);
	}
	else
	{
		Plate.DynamicMaterial = nullptr;
	}

	MarkRenderStateDirty();

#if WITH_EDITORONLY_DATA
	if (EditorFrustum)
	{
		EditorFrustum->MarkRenderStateDirty();
	}
#endif
}

FPrimitiveSceneProxy* UImagePlateComponent::CreateSceneProxy()
{
	return new FImagePlateSceneProxy(this);
}

UMaterialInterface* UImagePlateComponent::GetMaterial(int32 Index) const
{
	if (Index == 0)
	{
		return Plate.Material;
	}
	return nullptr;
}

void UImagePlateComponent::SetMaterial(int32 ElementIndex, UMaterialInterface* NewMaterial)
{
	if (ElementIndex == 0)
	{
		Plate.Material = NewMaterial;
		UpdateMaterialParametersForMedia();
	}
}

void UImagePlateComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	OutMaterials.AddUnique(Plate.DynamicMaterial ? Plate.DynamicMaterial : Plate.Material);
}

FBoxSphereBounds UImagePlateComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return FBoxSphereBounds(FVector(0,0,0), FVector(1,  1,  1), 1.73205f).TransformBy(LocalToWorld);
}

AActor* UImagePlateComponent::FindViewTarget() const
{
	AActor* Actor = GetOwner();
	while(Actor)
	{
		if (Actor->HasActiveCameraComponent() || Actor->FindComponentByClass<USceneCaptureComponent2D>())
		{
			return Actor;
		}
		Actor = Actor->GetAttachParentActor();
	}

	return nullptr;
}

#if WITH_EDITOR
void UImagePlateComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	UpdateMaterialParametersForMedia();
	UpdateTransformScale();
}

void UImagePlateComponent::PostEditUndo()
{
	Super::PostEditUndo();
	UpdateMaterialParametersForMedia();
}

FStructProperty* UImagePlateComponent::GetImagePlateProperty()
{
	return FindFProperty<FStructProperty>(StaticClass(), GET_MEMBER_NAME_CHECKED(UImagePlateComponent, Plate));
}

#endif
