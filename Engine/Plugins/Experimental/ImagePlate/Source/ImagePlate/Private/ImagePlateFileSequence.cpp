// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImagePlateFileSequence.h"
#include "HAL/FileManager.h"
#include "IImageWrapperModule.h"
#include "Modules/ModuleManager.h"
#include "RenderingThread.h"
#include "TextureResource.h"
#include "Misc/Paths.h"
#include "Async/Async.h"
#include "Misc/PackageName.h"
#include "Engine/Texture2DDynamic.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Textures/SlateTextureData.h"
#include "ImageUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImagePlateFileSequence)

DEFINE_LOG_CATEGORY_STATIC(LogImagePlateFileSequence, Log, Log);

UImagePlateFileSequence::UImagePlateFileSequence(const FObjectInitializer& Init)
	: Super(Init)
{
	FileWildcard = TEXT("*.exr");
	Framerate = 24;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		FModuleManager::LoadModuleChecked<IImageWrapperModule>("ImageWrapper");
	}
}

FImagePlateAsyncCache UImagePlateFileSequence::GetAsyncCache()
{
	FString Path;
	if (!FPackageName::TryConvertLongPackageNameToFilename(SequencePath.Path, Path))
	{
		UE_LOGF(LogImagePlateFileSequence, Warning, "Sequence path is not a long package name. This path is not portable, and may not work in a packaged build.");
		Path = SequencePath.Path;
	}

	if (GetDefault<UImagePlateSettings>()->ProxyName.Len() != 0)
	{
		FString ProxyPath = Path / GetDefault<UImagePlateSettings>()->ProxyName;
		if (FPaths::DirectoryExists(ProxyPath))
		{
			Path = MoveTemp(ProxyPath);
		}
	}

	return FImagePlateAsyncCache::MakeCache(Path, FileWildcard, Framerate);
}

FImagePlateSourceFrame::FImagePlateSourceFrame()
	: Width(0), Height(0), BitDepth(0), Pitch(0)
{}

FImagePlateSourceFrame::FImagePlateSourceFrame(const TArray64<uint8>& InData, uint32 InWidth, uint32 InHeight, uint32 InBitDepth)
	: Width(InWidth), Height(InHeight), BitDepth(InBitDepth), Pitch(Width * BitDepth / 8 * 4)
{
	if (InData.Num())
	{
		// Ideally we'd be able to steal the allocation from the array, but that's not possible with the current ImageWrapper/Array implementation
		Buffer = MakeShareable(new uint8[InData.Num()]);
		FMemory::Memcpy(Buffer.Get(), InData.GetData(), InData.Num());
	}
}

bool FImagePlateSourceFrame::IsValid() const
{
	return Buffer.IsValid();
}

bool FImagePlateSourceFrame::EnsureTextureMetrics(UTexture* DestinationTexture) const
{
	if (BitDepth != 16 && BitDepth != 8)
	{
		UE_LOGF(LogImagePlateFileSequence, Warning, "Unsupported source image bitdepth: %u. Only float 16 and fixed 8 bitdepths are supported", BitDepth);
		return false;
	}

	// Ensure the texture dimensions and bitdepth match if possible
	bool bNeedsUpdate = false;

	if (auto* Texture2DDynamic = Cast<UTexture2DDynamic>(DestinationTexture))
	{
		if (Width > 0 && Texture2DDynamic->SizeX != Width)
		{
			Texture2DDynamic->SizeX = Width;
			bNeedsUpdate = true;
		}

		if (Height > 0 && Texture2DDynamic->SizeY != Height)
		{
			Texture2DDynamic->SizeY = Height;
			bNeedsUpdate = true;
		}

		uint32 DestBitDepth = GPixelFormats[Texture2DDynamic->Format].BlockBytes * 8 / GPixelFormats[Texture2DDynamic->Format].NumComponents;
		if (DestBitDepth != BitDepth)
		{
			bNeedsUpdate = true;
			switch (BitDepth)
			{
			case 16: Texture2DDynamic->Format = PF_FloatRGBA; break;
			case 8: Texture2DDynamic->Format = PF_R8G8B8A8; break;
			}
		}
	}
	else if (auto* TextureRenderTarget2D = Cast<UTextureRenderTarget2D>(DestinationTexture))
	{
		if (Width > 0 && TextureRenderTarget2D->SizeX != Width)
		{
			TextureRenderTarget2D->SizeX = Width;
			bNeedsUpdate = true;
		}

		if (Height > 0 && TextureRenderTarget2D->SizeY != Height)
		{
			TextureRenderTarget2D->SizeY = Height;
			bNeedsUpdate = true;
		}

		EPixelFormat PF = GetPixelFormatFromRenderTargetFormat(TextureRenderTarget2D->RenderTargetFormat);
		uint32 DestBitDepth = GPixelFormats[PF].BlockBytes * 8 / GPixelFormats[PF].NumComponents;
		if (DestBitDepth != BitDepth)
		{
			bNeedsUpdate = true;
			switch (BitDepth)
			{
			case 16: TextureRenderTarget2D->RenderTargetFormat = RTF_RGBA16f; break;
			case 8: TextureRenderTarget2D->RenderTargetFormat = RTF_RGBA8; break;
			}
		}
	}
	else if (DestinationTexture->GetResource())
	{
		// We have to have a valid texture to check whether it's the right type or not
		if (!DestinationTexture->GetResource()->TextureRHI)
		{
			DestinationTexture->UpdateResource();
			FlushRenderingCommands();
		}

		FTextureRHIRef Texture2DRHI = DestinationTexture->GetResource()->TextureRHI ? DestinationTexture->GetResource()->TextureRHI->GetTexture2D() : nullptr;
		if (!Texture2DRHI)
		{
			UE_LOGF(LogImagePlateFileSequence, Warning, "Unsupported texture type encountered: Unable to update texture to fit source frame size or bitdepth.");
			return false;
		}
		else
		{
			// At least check the bitdepth
			uint32 DestBitDepth = GPixelFormats[Texture2DRHI->GetFormat()].BlockBytes * 8 / GPixelFormats[Texture2DRHI->GetFormat()].NumComponents;
			if (DestBitDepth != BitDepth)
			{
				UE_LOGF(LogImagePlateFileSequence, Warning, "Invalid destination texture bitdepth. Expected %u, encountered %u.", BitDepth, DestBitDepth);
				return false;
			}
		}
	}

	if (bNeedsUpdate)
	{
		DestinationTexture->UpdateResource();
	}
	return true;
}


TFuture<void> FImagePlateSourceFrame::CopyTo(UTexture* DestinationTexture)
{
	// Copy the data to the texture via a render command
	struct FRenderCommandData
	{
		TPromise<void> Promise;
		bool bClearTexture;
		FImagePlateSourceFrame SourceFrame;
		UTexture* DestinationTexture;
	};

	// Shared render command data to allow us to pass the promise through to the command
	typedef TSharedPtr<FRenderCommandData, ESPMode::ThreadSafe> FCommandDataPtr;
	FCommandDataPtr CommandData = MakeShared<FRenderCommandData, ESPMode::ThreadSafe>();

	TFuture<void> CompletionFuture = CommandData->Promise.GetFuture();

	CommandData->bClearTexture = !Buffer.IsValid() || !EnsureTextureMetrics(DestinationTexture);
	CommandData->SourceFrame = *this;
	CommandData->DestinationTexture = DestinationTexture;

	ENQUEUE_RENDER_COMMAND(CopySourceBufferToTexture)(
		[CommandData](FRHICommandListImmediate& RHICmdList)
		{
			if (!CommandData->DestinationTexture->GetResource() || !CommandData->DestinationTexture->GetResource()->TextureRHI)
			{
				return;
			}

			FTextureRHIRef Texture2DRHI = CommandData->DestinationTexture->GetResource()->TextureRHI->GetTexture2D();
			if (!Texture2DRHI)
			{
				return;
			}

			uint32 DestPitch = 0;
			uint8* DestinationBuffer = (uint8*)RHICmdList.LockTexture2D(Texture2DRHI, 0, RLM_WriteOnly, DestPitch, false);
			if (!DestinationBuffer)
			{
				UE_LOGF(LogImagePlateFileSequence, Warning, "Unable to lock texture for write");
				return;
			}

			if (CommandData->bClearTexture)
			{
				FMemory::Memzero(DestinationBuffer, DestPitch * Texture2DRHI->GetSizeY());
			}
			else
			{
				const uint8* SourceBuffer = CommandData->SourceFrame.Buffer.Get();
				const int32 MaxRow = FMath::Min(Texture2DRHI->GetSizeY(), CommandData->SourceFrame.Height);

				if (DestPitch == CommandData->SourceFrame.Pitch)
				{
					FMemory::Memcpy(DestinationBuffer, SourceBuffer, DestPitch * MaxRow);
				}
				else for (int32 Row = 0; Row < MaxRow; ++Row)
				{
					FMemory::Memcpy(DestinationBuffer, SourceBuffer, FMath::Min(CommandData->SourceFrame.Pitch, DestPitch));

					DestinationBuffer += DestPitch;
					SourceBuffer += CommandData->SourceFrame.Pitch;
				}
			}

			RHICmdList.UnlockTexture2D(Texture2DRHI, 0, false);

			CommandData->Promise.SetValue();
		}
	);

	return CompletionFuture;
}

TSharedRef<FSlateTextureData, ESPMode::ThreadSafe> FImagePlateSourceFrame::AsSlateTexture() const
{
	uint32 BytesPerPixel = (BitDepth * 4) / 8;
	TSharedRef<FSlateTextureData, ESPMode::ThreadSafe> TextureData = MakeShared<FSlateTextureData, ESPMode::ThreadSafe>(nullptr, Width, Height, BytesPerPixel);

	const uint8* SourceBuffer = Buffer.Get();
	uint8* DestinationBuffer = TextureData->GetRawBytesPtr();
	if (Pitch == Width * BytesPerPixel)
	{
		FMemory::Memcpy(DestinationBuffer, SourceBuffer, Width*Height*BytesPerPixel);
	}
	else for (uint32 Row = 0; Row < Height; ++Row)
	{
		FMemory::Memcpy(DestinationBuffer, SourceBuffer, FMath::Min(Width*BytesPerPixel, Pitch));

		DestinationBuffer += Pitch;
		SourceBuffer += (Width*BytesPerPixel);
	}

	return TextureData;
}

namespace ImagePlateFrameCache
{
	struct FPendingFrame;

	struct FCachedFrame
	{
		/** The frame number to which this frame relates */
		int32 FrameNumber;
		/** Optional promise that has been made to the data (stores the loaded result) */
		TOptional<TPromise<FImagePlateSourceFrame>> FrameData;
		/** Future that can be supplied to clients who wish to use this frame. Always valid where FrameData is */
		TSharedFuture<FImagePlateSourceFrame> Future;

		FCachedFrame(int32 InFrameNumber) : FrameNumber(InFrameNumber) {}

		~FCachedFrame()
		{
			// If something was waiting on this cached frame, we have to fulfil it with some empty data
			if (Future.IsValid() && !Future.IsReady())
			{
				FrameData->SetValue(FImagePlateSourceFrame());
			}
		}

		// non-copyable
		FCachedFrame(const FCachedFrame&)=delete;
		FCachedFrame& operator=(const FCachedFrame&)=delete;

		// moveable
		FCachedFrame(FCachedFrame&&)=default;
		FCachedFrame& operator=(FCachedFrame&&)=default;

		/** Get a future to this frame's data */
		TSharedFuture<FImagePlateSourceFrame> GetFrameData()
		{
			if (!FrameData.IsSet())
			{
				FrameData.Emplace();
				Future = FrameData->GetFuture().Share();
			}
			return Future;
		}

		/** Set this frame's data */
		void SetFrameData(FImagePlateSourceFrame In)
		{
			if (!FrameData.IsSet())
			{
				FrameData.Emplace();
				Future = FrameData->GetFuture().Share();
			}
			
			if (ensureAlways(!Future.IsReady()))
			{
				FrameData->SetValue(In);
			}
		}
	};

	/** Implementation of a simple frame cache */
	struct FImagePlateSequenceCache : TSharedFromThis<FImagePlateSequenceCache, ESPMode::ThreadSafe>
	{
		FImagePlateSequenceCache(const FString& InSequencePath, const FString& InWildcard, float Framerate);

		/** Set the current cached frame range, and return a future to the current frame's data */
		TSharedFuture<FImagePlateSourceFrame> GetPrecachedFrame(float Time, int32 LeadingPrecacheFrames, int32 TrailingPrecacheFrames);

		/** Set the frame data for the specified frame number */
		void SetFrameData(int32 FrameNumber, FImagePlateSourceFrame SourceData);

		/** Query for uncached frames that need to be loaded */
		void GetUncachedFrames(TArray<FPendingFrame>& OutFrames, int32 MaxToAdd);

		/** Return how many total frames there are in the sequence */
		int32 GetNumFrames() const;

		/** Specify that the supplied frame number is going to be loaded */
		void OnPreloadFrame(int32 FrameNumber);

	private:

		/** Array of frames that are pending load - only ever manipulated and read by the loader thread */
		TArray<int32> OutstandingFrameNumbers;

		// Critical section to guard against any threaded access to the class
		mutable FCriticalSection CriticalSection;
		
		/** Contiguous array of cached frames for the current range */
		TArray<FCachedFrame> CachedFrames;

		/** The frame number we're currently interested in */
		int32 CurrentFrameNumber;
		/** Cache range bounds */
		int32 MinCacheRange, MaxCacheRange;

		/** Contiguous array of frame filenames */
		TArray<FString> FrameFilenames;
		/** Framerate at which to display the above frames */
		float Framerate;
	}; 


	typedef TSharedRef<FImagePlateSequenceCache, ESPMode::ThreadSafe> FImagePlateSequenceCacheRef;
	typedef TSharedPtr<FImagePlateSequenceCache, ESPMode::ThreadSafe> FImagePlateSequenceCachePtr;

	/** Structure that denotes a frame to be cached */
	struct FPendingFrame
	{
		/** The frame number */
		int32 FrameNumber;
		/** The distance from the current frame time to this frame (used to prioritize frames that are closer to the current time) */
		int32 Offset;
		/** Frame filename - stored as a raw pointer as the array never reallocates, and a reference to the owner is stored below */
		const TCHAR* Filename;
		/** Owner of the frame */
		FImagePlateSequenceCacheRef Cache;
	};

	/** A request for a frame */
	struct FOngoingRequest
	{
		/** The frame number */
		int32 FrameNumber;
		/** Future to the data once loaded */
		TFuture<FImagePlateSourceFrame> Future;
		/** The cache that requested the load */
		TWeakPtr<FImagePlateSequenceCache, ESPMode::ThreadSafe> Cache;
	};

	FImagePlateSequenceCache::FImagePlateSequenceCache(const FString& InSequencePath, const FString& InWildcard, float InFramerate)
		: CurrentFrameNumber(-1), MinCacheRange(-1), MaxCacheRange(-1), Framerate(InFramerate)
	{
		FString SequenceFolder = InSequencePath;

		IFileManager::Get().FindFiles(FrameFilenames, *SequenceFolder, *InWildcard);
		if (FrameFilenames.Num() == 0)
		{
			UE_LOGF(LogImagePlateFileSequence, Warning, "The directory '%ls' does not contain any image files that match the wildcard '%ls'", *SequenceFolder, *InWildcard);
		}
		else
		{
			UE_LOGF(LogImagePlateFileSequence, Verbose, "Found %i image files in '%ls' with the wildcard '%ls'", FrameFilenames.Num(), *SequenceFolder, *InWildcard);
			FrameFilenames.Sort();
		}

		for (FString& Filename : FrameFilenames)
		{
			Filename = SequenceFolder / Filename;
		}
	}

	TSharedFuture<FImagePlateSourceFrame> FImagePlateSequenceCache::GetPrecachedFrame(float Time, int32 LeadingPrecacheFrames, int32 TrailingPrecacheFrames)
	{
		// Protect threaded access to the class
		FScopeLock Lock(&CriticalSection);

		// We assume the supplied time is already very close to a frame time, so round rather than floor
		const int32 NewFrameNumber = FMath::RoundToInt(Time * Framerate);
		if (!FrameFilenames.IsValidIndex(NewFrameNumber))
		{
			TPromise<FImagePlateSourceFrame> Promise;
			Promise.SetValue(FImagePlateSourceFrame());
			return Promise.GetFuture().Share();
		}

		CurrentFrameNumber = NewFrameNumber;

		const int32 PrevMinCacheRange = MinCacheRange;
		const int32 PrevMaxCacheRange = MaxCacheRange;

		MinCacheRange = FMath::Max(0,						CurrentFrameNumber - TrailingPrecacheFrames);
		MaxCacheRange = FMath::Min(FrameFilenames.Num()-1,	CurrentFrameNumber + LeadingPrecacheFrames);

		if (PrevMaxCacheRange == -1 || MaxCacheRange < PrevMinCacheRange || MinCacheRange > PrevMaxCacheRange)
		{
			// Not overlapping or no existing frames, just reset everything
			CachedFrames.Reset();
			for (int32 FrameNumber = MinCacheRange; FrameNumber <= MaxCacheRange; ++FrameNumber)
			{
				CachedFrames.Add(FCachedFrame(FrameNumber));
			}
		}
		else
		{
			// Overlapping range - preserve cached frames
			int32 FrontDifference = MinCacheRange - PrevMinCacheRange;
			int32 BackDifference = MaxCacheRange - PrevMaxCacheRange;

			if (FrontDifference > 0)
			{
				CachedFrames.RemoveAt(0, FrontDifference);
			}
			else for (int32 FrameNumber = PrevMinCacheRange - 1; FrameNumber >= MinCacheRange; --FrameNumber)
			{
				CachedFrames.Insert(FCachedFrame(FrameNumber), 0);
			}

			if (BackDifference < 0)
			{
				CachedFrames.RemoveAt(CachedFrames.Num() + BackDifference, FMath::Abs(BackDifference));
			}
			else for (int32 FrameNumber = PrevMaxCacheRange+1; FrameNumber <= MaxCacheRange; ++FrameNumber)
			{
				CachedFrames.Add(FCachedFrame(FrameNumber));
			}
		}

		// Return the current frame's data
		return CachedFrames[CurrentFrameNumber - MinCacheRange].GetFrameData();
	}

	void FImagePlateSequenceCache::SetFrameData(int32 FrameNumber, FImagePlateSourceFrame SourceData)
	{
		// Protect threaded access to the class
		FScopeLock Lock(&CriticalSection);

		int32 FrameIndex = FrameNumber - MinCacheRange;
		if (CachedFrames.IsValidIndex(FrameIndex))
		{
			CachedFrames[FrameIndex].SetFrameData(MoveTemp(SourceData));
		}
		OutstandingFrameNumbers.Remove(FrameNumber);
	}

	void FImagePlateSequenceCache::GetUncachedFrames(TArray<FPendingFrame>& OutFrames, int32 MaxToAdd)
	{
		// Protect threaded access to the class
		FScopeLock Lock(&CriticalSection);
		if (MinCacheRange == -1 || MaxCacheRange == -1)
		{
			return;
		}

		const int32 CurrentFrameIndex = CurrentFrameNumber - MinCacheRange;

		// Search forwards and backwards from the current frame number for the first uncached frame
		int32 ForwardIndex = CurrentFrameIndex;
		int32 BackwardIndex = CurrentFrameIndex;
		int32 NumCachedFrames = CachedFrames.Num();

		int32 NumAdded = 0;
		while (NumAdded <= MaxToAdd && (ForwardIndex < NumCachedFrames || BackwardIndex >= 0))
		{
			// Look forwards
			if (ForwardIndex < NumCachedFrames)
			{
				// If nothing has requested this yet, ask for it
				const FCachedFrame& Frame = CachedFrames[ForwardIndex];
				if (!OutstandingFrameNumbers.Contains(Frame.FrameNumber) && !Frame.Future.IsReady())
				{
					OutFrames.Add(FPendingFrame{ Frame.FrameNumber, FMath::Abs(Frame.FrameNumber - CurrentFrameIndex), *FrameFilenames[Frame.FrameNumber], AsShared() });
					++NumAdded;
				}
			}

			// Look backwards
			if (BackwardIndex != CurrentFrameIndex && BackwardIndex >= 0)
			{
				const FCachedFrame& Frame = CachedFrames[BackwardIndex];
				if (!OutstandingFrameNumbers.Contains(Frame.FrameNumber) && !Frame.Future.IsReady())
				{
					OutFrames.Add(FPendingFrame{ Frame.FrameNumber, FMath::Abs(Frame.FrameNumber - CurrentFrameIndex), *FrameFilenames[Frame.FrameNumber], AsShared() });
					++NumAdded;
				}
			}

			++ForwardIndex;
			--BackwardIndex;
		}
	}

	int32 FImagePlateSequenceCache::GetNumFrames() const
	{
		return FrameFilenames.Num();
	}

	void FImagePlateSequenceCache::OnPreloadFrame(int32 FrameNumber)
	{
		OutstandingFrameNumbers.AddUnique(FrameNumber);
	}

	/** Load an image file into CPU memory */
	FImagePlateSourceFrame LoadFileData(FString FilenameToLoad, int32 FrameNumber)
	{
		FImage Image;
		if ( ! FImageUtils::LoadImage(*FilenameToLoad,Image) )
		{
			UE_LOGF(LogImagePlateFileSequence, Warning, "Failed to load file data from '%ls'", *FilenameToLoad);
			return FImagePlateSourceFrame();
		}
		
		// this module only handles these formats :
		//case 16: Texture2DDynamic->Format = PF_FloatRGBA; break;
		//case 8: Texture2DDynamic->Format = PF_R8G8B8A8; break;
		int BitDepth;

		if ( ERawImageFormat::IsHDR(Image.Format) )
		{
			Image.ChangeFormat(ERawImageFormat::RGBA16F,EGammaSpace::Linear);
			BitDepth = 16;
		}
		else
		{
			Image.ChangeFormat(ERawImageFormat::BGRA8,EGammaSpace::sRGB);

			// this module wants RGBA not BGRA :
			FImageCore::TransposeImageRGBABGRA(Image);

			BitDepth = 8;
		}

		return FImagePlateSourceFrame(
			MoveTemp(Image.RawData),
			Image.SizeX,
			Image.SizeY,
			BitDepth);
	}

	class FFrameLoadingThread : FRunnable
	{
	public:
		FFrameLoadingThread()
		{
			Thread = FRunnableThread::Create(this, TEXT("FFrameLoadingThread"), 4 * 1024, TPri_AboveNormal);
		}

		TSharedRef<FImagePlateSequenceCache, ESPMode::ThreadSafe> InitializeLoader(const FString& SequenceFolder, const FString& Wildcard, float Framerate)
		{
			FScopeLock Lock(&CacheArrayCriticalSection);
			TSharedRef<FImagePlateSequenceCache, ESPMode::ThreadSafe> NewImpl = MakeShared<FImagePlateSequenceCache, ESPMode::ThreadSafe>(SequenceFolder, Wildcard, Framerate);
			ActiveCaches.Add(NewImpl);
			return NewImpl;
		}

	private:

		/** Check the specified array for completed requests */
		void ProcessCompletedRequests(const TArray<FImagePlateSequenceCacheRef>& Caches, TArray<FOngoingRequest>& Requests)
		{
			for (int32 Index = Requests.Num() - 1; Index >= 0; --Index)
			{
				FOngoingRequest& Request = Requests[Index];
				if (Requests[Index].Future.IsReady())
				{
					FImagePlateSequenceCachePtr Cache = Request.Cache.Pin();
					if (Cache.IsValid())
					{
						Cache->SetFrameData(Requests[Index].FrameNumber, Requests[Index].Future.Get());
					}

					Requests.RemoveAtSwap(Index, EAllowShrinking::No);
				}
			}
		}

		/** Remove any caches that are no longer valid */
		void RemoveStaleCaches()
		{
			FScopeLock Lock(&CacheArrayCriticalSection);
			for (int32 Index = ActiveCaches.Num() - 1; Index >= 0; --Index)
			{
				if (!ActiveCaches[Index].Pin().IsValid())
				{
					ActiveCaches.RemoveAtSwap(Index, EAllowShrinking::No);
				}
			}
		}

		/** Get a list of all active caches in a thread-safe manner */
		void GetAllActiveCaches(TArray<FImagePlateSequenceCacheRef>& OutCaches) const
		{
			TArray<FImagePlateSequenceCacheRef> AllCaches;

			FScopeLock Lock(&CacheArrayCriticalSection);
			for (auto& WeakCache : ActiveCaches)
			{
				TSharedPtr<FImagePlateSequenceCache, ESPMode::ThreadSafe> Cache = WeakCache.Pin();
				if (Cache.IsValid())
				{
					OutCaches.Add(Cache.ToSharedRef());
				}
			}
		}

		/** Run the thread */
		virtual uint32 Run()
		{
			static int32 MaxConcurrentLoads = 3;

			TArray<FOngoingRequest> Requests;
			TArray<FImagePlateSequenceCacheRef> AllCaches;

			for (;;)
			{
				// Reset our reference to the caches so we can remove stale ones if necessary
				AllCaches.Reset();

				// Remove any stale caches
				RemoveStaleCaches();

				// Get any active caches that are still open
				GetAllActiveCaches(AllCaches);

				// Process any completed requests
				ProcessCompletedRequests(AllCaches, Requests);

				// If we've no more capacity, yield
				const uint32 NumToAdd = MaxConcurrentLoads - Requests.Num();
				if (!NumToAdd)
				{
					FPlatformProcess::Sleep(0.f);
					continue;
				}

				// Gather any frames that need deserializing
				TArray<FPendingFrame> UncachedFrames;
				for (FImagePlateSequenceCacheRef& Cache : AllCaches)
				{
					Cache->GetUncachedFrames(UncachedFrames, NumToAdd);
				}

				// Sort by how far offset they are from the 'current' frame
				Algo::SortBy(UncachedFrames, &FPendingFrame::Offset);

				// Kick off as many requests as we can
				for (const FPendingFrame& UncachedFrame : UncachedFrames)
				{
					int32 FrameNumber = UncachedFrame.FrameNumber;
					FString FilenameToLoad = UncachedFrame.Filename;

					UncachedFrame.Cache->OnPreloadFrame(UncachedFrame.FrameNumber);

					TFuture<FImagePlateSourceFrame> Future = Async(
						EAsyncExecution::ThreadPool,
						[FilenameToLoad, FrameNumber]
						{
							return LoadFileData(FilenameToLoad, FrameNumber);
						}
					);

					Requests.Add(FOngoingRequest{ UncachedFrame.FrameNumber, MoveTemp(Future), UncachedFrame.Cache });
					if (Requests.Num() >= MaxConcurrentLoads)
					{
						break;
					}
				}
			}
		}

		/** Critical section to protect access to ActiveCaches */
		mutable FCriticalSection CacheArrayCriticalSection;
		/** Weak list of caches that we have opened. Periodically purged of stale items. */
		TArray<TWeakPtr<FImagePlateSequenceCache, ESPMode::ThreadSafe>> ActiveCaches;

		/** The thread that's running us */
		FRunnableThread* Thread;
	};

	FFrameLoadingThread& GetFrameLoader()
	{
		static FFrameLoadingThread Thread;
		return Thread;
	}

}	// namespace ImagePlateFrameCache

FImagePlateAsyncCache FImagePlateAsyncCache::MakeCache(const FString& InSequencePath, const FString& InWildcard, float Framerate)
{
	FImagePlateAsyncCache NewCache;
	NewCache.Impl = ImagePlateFrameCache::GetFrameLoader().InitializeLoader(InSequencePath, InWildcard, Framerate);
	return NewCache;
}

TSharedFuture<FImagePlateSourceFrame> FImagePlateAsyncCache::RequestFrame(float Time, int32 LeadingPrecacheFrames, int32 TrailingPrecacheFrames)
{
	return Impl->GetPrecachedFrame(Time, LeadingPrecacheFrames, TrailingPrecacheFrames);
}

int32 FImagePlateAsyncCache::Length() const
{
	return Impl->GetNumFrames();
}

