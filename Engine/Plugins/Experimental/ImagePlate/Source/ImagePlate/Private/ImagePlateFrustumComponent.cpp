// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DrawFrsutumComponent.cpp: UDrawFrsutumComponent implementation.
=============================================================================*/

#include "ImagePlateFrustumComponent.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "Engine/CollisionProfile.h"
#include "MeshElementCollector.h"
#include "PrimitiveDrawingUtils.h"
#include "ImagePlateComponent.h"
#include "SceneView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImagePlateFrustumComponent)


/** Represents a draw frustum to the scene manager. */
class FImagePlateFrustumSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FImagePlateFrustumSceneProxy(const UImagePlateFrustumComponent* InComponent)
		: FPrimitiveSceneProxy(InComponent)
	{
		bWillEverBeLit = false;

		UImagePlateComponent* Parent = Cast<UImagePlateComponent>(InComponent->GetAttachParent());
		if (Parent && Parent->GetPlate().bFillScreen)
		{
			if (AActor* ViewTarget = Parent->FindViewTarget())
			{
				ViewTargetID = ViewTarget->GetUniqueID();
			}
			InvViewProjectionMatrix = Parent->GetCachedInvViewProjectionMatrix();
		}
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		if (!ViewTargetID)
		{
			return;
		}

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (!(VisibilityMap & (1 << ViewIndex)))
			{
				continue;
			}

			const FSceneView* View = Views[ViewIndex];

			// Don't draw frustums on views that are set as the current view target
			if (View->ViewActor.ActorUniqueId == ViewTargetID)
			{
				continue;
			}

			// These are all in world space
			const float NearPlaneZ = 1.f;
			FVector NearPlaneTopLeft 		= UImagePlateComponent::TransfromFromProjection(InvViewProjectionMatrix, FVector4(-1.f, -1.f, NearPlaneZ, 1.0f));
			FVector NearPlaneTopRight 		= UImagePlateComponent::TransfromFromProjection(InvViewProjectionMatrix, FVector4( 1.f, -1.f, NearPlaneZ, 1.0f));
			FVector NearPlaneBottomLeft 	= UImagePlateComponent::TransfromFromProjection(InvViewProjectionMatrix, FVector4(-1.f,  1.f, NearPlaneZ, 1.0f));
			FVector NearPlaneBottomRight	= UImagePlateComponent::TransfromFromProjection(InvViewProjectionMatrix, FVector4( 1.f,  1.f, NearPlaneZ, 1.0f));

			FVector DestinationTopLeft 		=  GetLocalToWorld().TransformPosition(FVector(0.f, -1.f, -1.f));
			FVector DestinationTopRight 	=  GetLocalToWorld().TransformPosition(FVector(0.f,  1.f, -1.f));
			FVector DestinationBottomLeft 	=  GetLocalToWorld().TransformPosition(FVector(0.f, -1.f,  1.f));
			FVector DestinationBottomRight	=  GetLocalToWorld().TransformPosition(FVector(0.f,  1.f,  1.f));

			FColor LineColor(255, 0, 255, 128);

			FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);
			const uint8 DepthPriorityGroup = GetDepthPriorityGroup(View);
			PDI->DrawLine(NearPlaneTopLeft, DestinationTopLeft, LineColor, DepthPriorityGroup );
			PDI->DrawLine(NearPlaneTopRight, DestinationTopRight, LineColor, DepthPriorityGroup );
			PDI->DrawLine(NearPlaneBottomLeft, DestinationBottomLeft, LineColor, DepthPriorityGroup );
			PDI->DrawLine(NearPlaneBottomRight, DestinationBottomRight, LineColor, DepthPriorityGroup );
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bDynamicRelevance = true;
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
		return Result;
	}

	virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }
	uint32 GetAllocatedSize( void ) const { return( FPrimitiveSceneProxy::GetAllocatedSize() ); }

private:
	uint32 ViewTargetID = 0;
	FMatrix InvViewProjectionMatrix;
};

UImagePlateFrustumComponent::UImagePlateFrustumComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUseEditorCompositing = true;
	bHiddenInGame = true;
	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	SetGenerateOverlapEvents(false);
}

FPrimitiveSceneProxy* UImagePlateFrustumComponent::CreateSceneProxy()
{
	return new FImagePlateFrustumSceneProxy(this);
}

FBoxSphereBounds UImagePlateFrustumComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	UImagePlateComponent* Parent = Cast<UImagePlateComponent>(GetAttachParent());
	
	FBox MaxBox(TArray<FVector>{
		FVector(0,  1,  1),
		FVector(0, -1,  1),
		FVector(0,  1, -1),
		FVector(0, -1, -1),
		});

	MaxBox = MaxBox.TransformBy(LocalToWorld);

	// Include the near view plane if possible
	if (Parent && Parent->GetPlate().bFillScreen)
	{
		const FMatrix& InvViewProjectionMatrix = Parent->GetCachedInvViewProjectionMatrix();
		
		// These are all in world space
		const float NearPlaneZ = 1.f;
		MaxBox += UImagePlateComponent::TransfromFromProjection(InvViewProjectionMatrix, FVector4(0.f, 0.f, NearPlaneZ, 1.0f));
	}

	return FBoxSphereBounds(MaxBox);
}
