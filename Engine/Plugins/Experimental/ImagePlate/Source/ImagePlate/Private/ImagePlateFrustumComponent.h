// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "ImagePlateFrustumComponent.generated.h"


/** 
 * A 2d material that will be rendered always facing the camera.
 */
UCLASS(MinimalAPI, ClassGroup=Rendering, hidecategories=(Object,Activation,Collision,"Components|Activation",Physics), editinlinenew)
class UImagePlateFrustumComponent : public UPrimitiveComponent
{
public:
	GENERATED_BODY()

	UImagePlateFrustumComponent(const FObjectInitializer& Init);

	//~ Begin UPrimitiveComponent Interface
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	virtual bool ShouldRecreateProxyOnUpdateTransform() const override { return true; }
};
