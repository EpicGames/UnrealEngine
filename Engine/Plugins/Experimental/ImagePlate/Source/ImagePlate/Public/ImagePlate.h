// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "ImagePlate.generated.h"

#define UE_API IMAGEPLATE_API

class UImagePlateComponent;

UCLASS(MinimalAPI, ClassGroup=Rendering, hidecategories=(Object,Activation,Physics,Collision,Input,PhysicsVolume))
class AImagePlate : public AActor
{
public:
	GENERATED_BODY()

	UE_API AImagePlate(const FObjectInitializer& Init);

	UImagePlateComponent* GetImagePlateComponent() const
	{
		return ImagePlate;
	}

protected:

	UPROPERTY(Category=ImagePlate, VisibleAnywhere, BlueprintReadOnly, meta=(ExposeFunctionCategories="Mesh,Rendering,Physics,Components|StaticMesh"))
	TObjectPtr<UImagePlateComponent> ImagePlate;
};

#undef UE_API
