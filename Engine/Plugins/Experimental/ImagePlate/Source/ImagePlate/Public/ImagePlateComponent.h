// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "Physics/PhysicsInterfaceTypes.h"
#include "ImagePlateComponent.generated.h"

#define UE_API IMAGEPLATE_API

class UMaterialInstanceDynamic;

class FPrimitiveSceneProxy;
class UImagePlateFrustumComponent;
class UMaterialInterface;
class UTexture;

USTRUCT(BlueprintType)
struct FImagePlateParameters
{
	GENERATED_BODY()

	UE_API FImagePlateParameters();

	/** The material that the image plate is rendered with. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category="Image Plate")
	TObjectPtr<UMaterialInterface> Material;

	/** Name of a texture parameter inside the material to patch the render target texture to */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category="Image Plate")
	FName TextureParameterName;

	/** Automatically size the plate based on the active camera's lens and filmback settings. Target Camera is found by looking for an active camera component from this component's actor, through its attached parents. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Image Plate")
	bool bFillScreen;

	/** The amount to fill the screen with when attached to a camera */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Image Plate", meta=(EditCondition="bFillScreen", AllowPreserveRatio))
	FVector2D FillScreenAmount;

	/** The fixed size of the image plate */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Image Plate", meta=(EditCondition="!bFillScreen"))
	FVector2D FixedSize;

	/** Transient texture that receives image frames */
	UPROPERTY(transient, BlueprintReadOnly, Category="Image Plate")
	TObjectPtr<UTexture> RenderTexture;

	/** Transient MID to hold the material with the render texture patched in */
	UPROPERTY(transient)
	TObjectPtr<UMaterialInstanceDynamic> DynamicMaterial;
};

/** 
 * A 2d plate that will be rendered always facing the camera.
 */
UCLASS(MinimalAPI, ClassGroup=Rendering, hidecategories=(Object,Activation,Collision,"Components|Activation",Physics), editinlinenew, meta=(BlueprintSpawnableComponent))
class UImagePlateComponent : public UPrimitiveComponent
{
public:

	GENERATED_BODY()
	UE_API UImagePlateComponent(const FObjectInitializer& Init);

	static inline FVector TransfromFromProjection(const FMatrix& Matrix, const FVector4& InVector)
	{
		FVector4 HomogenousVector = Matrix.TransformFVector4(InVector);
		FVector ReturnVector = HomogenousVector;
		if (HomogenousVector.W != 0.0f)
		{
			ReturnVector /= HomogenousVector.W;
		}

		return ReturnVector;
	}

	/** Add an image plate to this actor */
	UFUNCTION(BlueprintCallable, Category="Game|Image Plate")
	UE_API void SetImagePlate(FImagePlateParameters Plate);

	/** Get this actor's image plates */
	UFUNCTION(BlueprintCallable, Category="Game|Image Plate")
	FImagePlateParameters GetPlate() const
	{
		return Plate;
	}

	/** Called by sequencer if a texture is changed */
	UFUNCTION()
	UE_API void OnRenderTextureChanged();

	/** Access this component's cached view projection matrix. Only valid when the plate is set to fill screen */
	const FMatrix& GetCachedViewProjectionMatrix() const
	{
		return ViewProjectionMatrix;
	}

	/** Access this component's cached inverse view projection matrix. Only valid when the plate is set to fill screen */
	const FMatrix& GetCachedInvViewProjectionMatrix() const
	{
		return InvViewProjectionMatrix;
	}

	//~ Begin UPrimitiveComponent Interface
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UE_API virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport) override;
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	UE_API virtual UMaterialInterface* GetMaterial(int32 Index) const override;
	UE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const override;
	UE_API virtual void SetMaterial(int32 ElementIndex, UMaterialInterface* Material) override;
	UE_API virtual void OnRegister() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;
#endif

#if WITH_EDITOR
	/** Access the property relating to this component's image plate */
	static UE_API FStructProperty* GetImagePlateProperty();
#endif

	/**
	 * Finds a view target that this image plate is presenting to
	 */
	UE_API AActor* FindViewTarget() const;

protected:

	UE_API void UpdateMaterialParametersForMedia();

	UE_API void UpdateTransformScale();

private:

	/** Array of image plates to render for this component */
	UPROPERTY(EditAnywhere, Category="Image Plate", DisplayName="Image Plates", meta=(ShowOnlyInnerProperties))
	FImagePlateParameters Plate;

	FMatrix ViewProjectionMatrix;
	FMatrix InvViewProjectionMatrix;

	bool bReenetrantTranformChange;

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	TObjectPtr<UImagePlateFrustumComponent> EditorFrustum;
#endif
};

#undef UE_API
