// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImagePlateFileSequence.generated.h"

#define UE_API IMAGEPLATE_API

template <typename ResultType> class TFuture;
template <typename ResultType> class TSharedFuture;

class UTexture;

struct FSlateTextureData;
struct FImagePlateAsyncCache;
namespace ImagePlateFrameCache { struct FImagePlateSequenceCache; }

/**
* Implements the settings for the ImagePlate plugin.
*/
UCLASS(MinimalAPI, config=Engine, defaultconfig)
class UImagePlateSettings : public UObject
{
public:
	GENERATED_BODY()

	/** Specifies a sub-directory to append to any image plate file sequences */
	UPROPERTY(GlobalConfig, EditAnywhere, config, Category=Settings)
	FString ProxyName;
};


UCLASS(MinimalAPI)
class UImagePlateFileSequence : public UObject
{
public:
	GENERATED_BODY()

	UE_API UImagePlateFileSequence(const FObjectInitializer& Init);

	/** Path to the directory in which the image sequence resides */
	UPROPERTY(EditAnywhere, Category="General", meta=(ContentDir))
	FDirectoryPath SequencePath;

	/** Wildcard used to find images within the directory (ie *.exr) */
	UPROPERTY(EditAnywhere, Category="General")
	FString FileWildcard;

	/** Framerate at which to display the images */
	UPROPERTY(EditAnywhere, Category="General", meta=(ClampMin=0))
	float Framerate;

public:

	/** Create a new image cache for this sequence */
	UE_API FImagePlateAsyncCache GetAsyncCache();
};

/** Uncompressed source data for a single frame of a sequence */
struct FImagePlateSourceFrame
{
	/** Default constructor */
	UE_API FImagePlateSourceFrame();
	/** Construction from an array of data, and a given width/height/bitdepth */
	UE_API FImagePlateSourceFrame(const TArray64<uint8>& InData, uint32 InWidth, uint32 InHeight, uint32 InBitDepth);

	/** Check whether this source frame has valid data */
	UE_API bool IsValid() const;

	/** Copy the contents of this frame to the specified texture */
	UE_API TFuture<void> CopyTo(UTexture* DestinationTexture);

	/** Copy this source frame into a slate texture data format */
	UE_API TSharedRef<FSlateTextureData, ESPMode::ThreadSafe> AsSlateTexture() const;

private:

	/** Ensure the specified texture metrics match this frame */
	bool EnsureTextureMetrics(UTexture* DestinationTexture) const;

	/** Metrics for the texture */
	uint32 Width, Height, BitDepth, Pitch;

	/** Threadsafe, shared data buffer. Shared so that this type can be copied around without incurring a copy-cost for large frames. */
	TSharedPtr<uint8, ESPMode::ThreadSafe> Buffer;
};

/** A wrapper for an asynchronous cache of image frames */
struct FImagePlateAsyncCache
{
	/** Make a new cache for the specified folder, wildcard and framerate */
	static UE_API FImagePlateAsyncCache MakeCache(const FString& InSequencePath, const FString& InWildcard, float Framerate);

	/** Request a frame of data from the cache, whilst also caching leading and trailing frames if necessary */
	// @todo: sequencer-timecode: frame accuracy
	UE_API TSharedFuture<FImagePlateSourceFrame> RequestFrame(float Time, int32 LeadingPrecacheFrames, int32 TrailingPrecacheFrames);

	/** Get the length of the sequence in frames */
	UE_API int32 Length() const;

private:
	FImagePlateAsyncCache(){}

	/** Shared implementation */
	TSharedPtr<ImagePlateFrameCache::FImagePlateSequenceCache, ESPMode::ThreadSafe> Impl;
};

#undef UE_API
