// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ImagePlateEditor : ModuleRules
	{
		public ImagePlateEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"Core",
					"CoreUObject",
					"ImagePlate",
					"Settings",
					"SlateCore",
					"UnrealEd",
				}
			);
		}
	}
}
