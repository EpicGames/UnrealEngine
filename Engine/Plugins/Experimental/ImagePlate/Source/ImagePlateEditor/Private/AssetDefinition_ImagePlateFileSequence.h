// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImagePlateFileSequence.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_ImagePlateFileSequence.generated.h"

UCLASS()
class UAssetDefinition_ImagePlateFileSequence : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ImagePlateFileSequence", "Image Plate File Sequence"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(105, 0, 60)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UImagePlateFileSequence::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_ImagePlateFileSequenceSubMenu", "Other"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
