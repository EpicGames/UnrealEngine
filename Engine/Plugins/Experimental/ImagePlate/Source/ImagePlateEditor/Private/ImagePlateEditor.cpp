// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImagePlateFileSequence.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyle.h"
#include "Misc/Paths.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateStyleRegistry.h"
#include "ISettingsModule.h"

#define LOCTEXT_NAMESPACE "ImagePlateEditor"

class FImagePlateEditorStyle final : public FSlateStyleSet
{
public:
	FImagePlateEditorStyle() : FSlateStyleSet("ImagePlateEditorStyle")
	{
		const FVector2D Icon16x16(16.f, 16.f);
		const FVector2D Icon64x64(64.f, 64.f);

		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/ImagePlate/Resources/Icons"));

		Set("ClassIcon.ImagePlate", new FSlateImageBrush(RootToContentDir(TEXT("ImagePlate_16x.png")), Icon16x16));
		Set("ClassThumbnail.ImagePlate", new FSlateImageBrush(RootToContentDir(TEXT("ImagePlate_64x.png")), Icon64x64));

		Set("ClassIcon.ImagePlateComponent", new FSlateImageBrush(RootToContentDir(TEXT("ImagePlate_16x.png")), Icon16x16));
		Set("ClassThumbnail.ImagePlateComponent", new FSlateImageBrush(RootToContentDir(TEXT("ImagePlate_64x.png")), Icon64x64));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	~FImagePlateEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

public:

	static FImagePlateEditorStyle& Get()
	{
		if (!Singleton.IsSet())
		{
			Singleton.Emplace();
		}
		return Singleton.GetValue();
	}
	
	static void Destroy()
	{
		Singleton.Reset();
	}

private:
	static TOptional<FImagePlateEditorStyle> Singleton;
};

TOptional<FImagePlateEditorStyle> FImagePlateEditorStyle::Singleton;

class FImagePlateEditorModule : public IModuleInterface
{
public:

	virtual void StartupModule() override
	{
		FImagePlateEditorStyle::Get();

		ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
		SettingsModule.RegisterSettings("Project", "Plugins", "ImagePlate",
			LOCTEXT("ImagePlateEditorSettingsName", "Image Plate"),
			LOCTEXT("ImagePlateEditorSettingsDescription", "Configure settings for the Image Plate plugin."),
			GetMutableDefault<UImagePlateSettings>()
		);
	}
	
	virtual void ShutdownModule() override
	{
		FImagePlateEditorStyle::Destroy();
		
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "ImagePlate");
		}
	}
};

IMPLEMENT_MODULE(FImagePlateEditorModule, ImagePlateEditor);

#undef LOCTEXT_NAMESPACE
