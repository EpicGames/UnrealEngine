// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImagePlateFileSequenceFactory.h"
#include "ImagePlateFileSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImagePlateFileSequenceFactory)

#define LOCTEXT_NAMESPACE "ImagePlateFileSequenceFactory"

UImagePlateFileSequenceFactory::UImagePlateFileSequenceFactory(const FObjectInitializer& ObjectInitializer)
 : Super(ObjectInitializer)
{
	bCreateNew = true;
	SupportedClass = UImagePlateFileSequence::StaticClass();
}

FText UImagePlateFileSequenceFactory::GetDisplayName() const
{
	return LOCTEXT("ImagePlateFileSequenceFactoryDescription", "Image Plate File Sequence");
}

UObject* UImagePlateFileSequenceFactory::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	return NewObject<UImagePlateFileSequence>(InParent, Name, Flags);
}

#undef LOCTEXT_NAMESPACE

