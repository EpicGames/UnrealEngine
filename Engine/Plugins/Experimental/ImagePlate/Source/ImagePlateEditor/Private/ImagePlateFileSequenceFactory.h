// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "ImagePlateFileSequenceFactory.generated.h"

UCLASS(MinimalAPI)
class UImagePlateFileSequenceFactory : public UFactory
{
	GENERATED_BODY()

	UImagePlateFileSequenceFactory(const FObjectInitializer& ObjectInitializer);

	virtual FText GetDisplayName() const override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
};
